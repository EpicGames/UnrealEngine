{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Skeletal Mesh Morph Target Editing Tools",
	"Description": "Tools to edit morph targets within the skeletal mesh editor.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com/",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "SkeletalMeshMorphTargetEditingTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "SkeletalMeshModelingTools",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}