// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"
#include "EraseMorphTargetBrushOps.generated.h"


UCLASS(MinimalAPI)
class UEraseMorphTargetBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = InflateBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = InflateBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

class FEraseMorphTargetBrushOp : public FMeshSculptBrushOp
{

public:
	double BrushSpeedTuning = 2.0;

	typedef TUniqueFunction<const FDynamicMesh3*()> GetMeshWithoutCurrentMorphFuncType;

	GetMeshWithoutCurrentMorphFuncType GetMeshWithoutCurrentMorphFunc;
	
	FEraseMorphTargetBrushOp(GetMeshWithoutCurrentMorphFuncType InFunc)
	{
		GetMeshWithoutCurrentMorphFunc = MoveTemp(InFunc);
	}
	
	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::SculptMesh;
	}

	// Match FSingleNormalSculptBrushOp (Offset): Spacing-gated emission so a stationary cursor
	// does not keep firing stamps.
	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		const FDynamicMesh3* MeshWithoutCurentMorph = GetMeshWithoutCurrentMorphFunc();
		
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		double UsePower = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);
			FVector3d TargetPos = MeshWithoutCurentMorph->GetVertex(VertIdx);
			FVector3d MaxDelta = TargetPos - OrigPos; 

			FVector3d MoveVec = UE::Geometry::Normalized(TargetPos - OrigPos, FMathd::ZeroTolerance) * UsePower;

			double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);

			MoveVec = MoveVec * Falloff;
			MoveVec = MoveVec.GetClampedToMaxSize(MaxDelta.Length());
			
			FVector3d NewPos = OrigPos + MoveVec;
			NewPositionsOut[k] = NewPos;
		});
	}

};

