// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMorphTargetEditingToolInterface.h"

#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IMorphTargetEditingToolInterface)

void IMorphTargetEditingToolInterface::SetupMorphEditingToolCommon()
{
	UInteractiveTool* Tool = CastChecked<UInteractiveTool>(this);
	USkeletalMeshEditorContextObjectBase* EditorContext = Tool->GetToolManager()->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();

	
	auto SetupFunction = [&](UMorphTargetEditingToolProperties* InProperties)
	{
		InProperties->EditMorphTargetName = EditorContext->GetEditingMorphTarget();
	};
	
	SetupCommonProperties(SetupFunction);
}

void IMorphTargetEditingToolInterface::ShutdownMorphEditingToolCommon()
{

}
