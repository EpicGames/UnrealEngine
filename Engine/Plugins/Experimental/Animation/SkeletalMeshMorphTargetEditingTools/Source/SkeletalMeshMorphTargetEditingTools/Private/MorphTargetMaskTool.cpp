// Copyright Epic Games, Inc. All Rights Reserved.

#include "MorphTargetMaskTool.h"

#include "ContextObjectStore.h"
#include "Editor.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "MorphTargetVertexSculptTool.h"
#include "PreviewMesh.h"
#include "ReferenceSkeleton.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "Components/DynamicMeshComponent.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MorphTargetMaskTool)

#define LOCTEXT_NAMESPACE "MorphTargetMaskTool"

const TCHAR* UMorphTargetMaskToolProperties::CreateNewVertexAttributeOption = TEXT("[ Save new Mask ]");
const TCHAR* UMorphTargetMaskToolProperties::DoNotSaveVertexAttributeOption = TEXT("[ Temporary Mask ]");

bool UMorphTargetMaskToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	USkeletalMeshEditorContextObjectBase* EditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();
	if (!EditorContext)
	{
		return false;
	}

	if (EditorContext->GetEditingMorphTarget() == NAME_None)
	{
		return false;
	}
	
	return Super::CanBuildTool(SceneState);
}

UMeshSurfacePointTool* UMorphTargetMaskToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMorphTargetMaskTool* MorphTargetEditorTool = NewObject<UMorphTargetMaskTool>(SceneState.ToolManager);
	MorphTargetEditorTool->SetWorld(SceneState.World);
	return MorphTargetEditorTool;
}

const FToolTargetTypeRequirements& UMorphTargetMaskToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements ToolRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		USceneComponentBackedTarget::StaticClass(),
		USkeletonProvider::StaticClass(),
	});

	return ToolRequirements;
}

void UMorphTargetMaskToolProperties::Setup(UMorphTargetMaskTool* InTool, TArray<FName> InVertexAttributes)
{
	SetFlags(RF_Transactional);
	
	// Setup Tool Settings
	FString AssetPath = TEXT("UnknownAsset");
	UToolTarget* Target = InTool->GetTarget();
	if (Cast<ISkeletalMeshBackedTarget>(Target))
	{
		if (USkeletalMesh* SkeletalMesh = Cast<ISkeletalMeshBackedTarget>(Target)->GetSkeletalMesh())
		{
			AssetPath = SkeletalMesh->GetPathName();
		}
	}
	
	PropertySetCacheIdentifier = FString::Printf(TEXT("%s::%s"), *AssetPath, *InTool->GetEditingMorphTarget().ToString());
	RestoreProperties(InTool, PropertySetCacheIdentifier); 
	
	InTool->GetMorphTargetWeights().GenerateKeyArray(MorphTargets);
	if (!MorphTargets.Contains(BaseMorphTarget))
	{
		BaseMorphTarget = InTool->GetEditingMorphTarget();
	}

	VertexAttributeOptions.Add(CreateNewVertexAttributeOption);
	VertexAttributeOptions.Add(DoNotSaveVertexAttributeOption);
	VertexAttributeOptions.Append(InVertexAttributes);
	VertexAttributes = MoveTemp(InVertexAttributes);


	if (VertexAttributeOption != CreateNewVertexAttributeOption && VertexAttributeOption != DoNotSaveVertexAttributeOption)
	{
		if (!VertexAttributes.Contains(VertexAttributeOption))
		{
			VertexAttributeOption = CreateNewVertexAttributeOption;
		}
	}
}

void UMorphTargetMaskToolProperties::Shutdown(UMorphTargetMaskTool* InTool , EToolShutdownType InShutdownType)
{
	if (InShutdownType != EToolShutdownType::Cancel)
	{
		SaveProperties(InTool, PropertySetCacheIdentifier);
	}
}

bool UMorphTargetMaskToolProperties::ShouldSaveMask() const
{
	return VertexAttributeOption != DoNotSaveVertexAttributeOption;
}

bool UMorphTargetMaskToolProperties::ShouldSaveNewMask() const
{
	return VertexAttributeOption == CreateNewVertexAttributeOption;
}

const TArray<FName>& UMorphTargetMaskToolProperties::GetMorphTargets()
{
	return MorphTargets;
}

const TArray<FName>& UMorphTargetMaskToolProperties::GetVertexAttributeOptions()
{
	return VertexAttributeOptions;
}

TOptional<int32> UMorphTargetMaskToolProperties::GetVertexAttributeIndex()
{
	if (VertexAttributeOption != CreateNewVertexAttributeOption && 
		VertexAttributeOption != DoNotSaveVertexAttributeOption)
	{
		return VertexAttributes.IndexOfByKey(VertexAttributeOption);
	}

	return {};
}

bool UMorphTargetMaskTool::SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex)
{
	using namespace UE::Geometry;

	TArray<FName> VertexAttributes;
	for (int32 WeightMapIndex = 0; WeightMapIndex < InOutToolMesh.Attributes()->NumWeightLayers(); WeightMapIndex++)
	{
		FName AttributeName = InOutToolMesh.Attributes()->GetWeightLayer(WeightMapIndex)->GetName();
		VertexAttributes.Add(AttributeName);
	}
	
	MaskToolProperties->Setup(this, MoveTemp(VertexAttributes));
	
	MaskToolCommitMesh = InOutToolMesh;
	{
		// Create a temp copy of the existing morph target so that a weight map can be applied to it to create the new replacement morph target 
		FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = MaskToolCommitMesh.Attributes()->GetMorphTargetAttribute(ToolMorphTargetName);

		ToolMorphTargetBackupAttributeName = TEXT("MorphTargetMaskTool_ToolMorphTargetBackup");

		while (MaskToolCommitMesh.Attributes()->HasMorphTargetAttribute(ToolMorphTargetBackupAttributeName))
		{
			ToolMorphTargetBackupAttributeName.SetNumber(ToolMorphTargetBackupAttributeName.GetNumber() + 1);
		}
	
		FDynamicMeshMorphTargetAttribute* ToolMorphTargetBackupAttribute = new FDynamicMeshMorphTargetAttribute(&MaskToolCommitMesh);
	
		ToolMorphTargetBackupAttribute->Copy(*MorphTargetAttribute);
		ToolMorphTargetBackupAttribute->SetName(ToolMorphTargetBackupAttributeName);
			
		MaskToolCommitMesh.Attributes()->AttachMorphTargetAttribute(ToolMorphTargetBackupAttributeName, ToolMorphTargetBackupAttribute);	
	}

	// Build isolated submesh if isolation was active
	if (EditorContext.IsValid())
	{
		const TArray<int32>& IsolatedTriangles = EditorContext->GetIsolatedTriangles();
		if (!IsolatedTriangles.IsEmpty())
		{
			IsolationSubmesh = FDynamicSubmesh3(&MaskToolCommitMesh, IsolatedTriangles);
			InOutToolMesh = IsolationSubmesh->GetSubmesh();
		}
	}

	// Create a temp weight layer to paint into
	const int32 NumAttributeLayers = InOutToolMesh.Attributes()->NumWeightLayers();
	InOutToolMesh.Attributes()->SetNumWeightLayers(NumAttributeLayers + 1);
	FDynamicMeshWeightAttribute* ActiveWeightMap = InOutToolMesh.Attributes()->GetWeightLayer(NumAttributeLayers);
	ActiveWeightMap->SetName(FName("PaintLayer"));

	TempPaintLayerAttributeIndex = NumAttributeLayers;

	
	if (TOptional<int32> AttributeIndex = MaskToolProperties->GetVertexAttributeIndex())
	{
		OutInitialAttributeIndex = *AttributeIndex;
	}
	else
	{
		OutInitialAttributeIndex = TempPaintLayerAttributeIndex;
	}

	CurrentAttributeIndex = OutInitialAttributeIndex;
	
	return true;
}



namespace UE::MorphTargetMaskTool::Private
{
	class SSaveMaskWindow : public SCompoundWidget
	{
	public:
		struct FResult
		{
			FText AttributeName;
			bool bSave = false;
		};
		
		SLATE_BEGIN_ARGS(SSaveMaskWindow) {}
			SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
			SLATE_ARGUMENT( TArray<FName>, ExistingNames )
			SLATE_ARGUMENT( FResult*, ResultPtr)
		SLATE_END_ARGS()

		virtual ~SSaveMaskWindow() = default;
		
		void Construct(const FArguments& InArgs)
		{
			WidgetWindow = InArgs._WidgetWindow;
			ExistingNames = InArgs._ExistingNames;
			ResultPtr = InArgs._ResultPtr;

			this->ChildSlot
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(6.f, 0.0))
					[
						SNew(STextBlock)
						.Text(FText::FromString("Mask Name: "))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(6.f, 0.0))
					[
						SNew(SEditableTextBox)
						.Text_Lambda([this]()
							{
								return ResultPtr->AttributeName;
							})
						.OnTextChanged_Lambda([this](const FText& InText)
							{
								ResultPtr->AttributeName = InText;
							})
						.OnVerifyTextChanged_Lambda([this](const FText&, FText& OutErrorMessage)
							{
								FName Name = *ResultPtr->AttributeName.ToString();
								if (ExistingNames.Contains(Name))
								{
									OutErrorMessage = FText::FromString(TEXT("The name is already used by another attribute"));
									return false;
								}

								return true;
							})
						.OnTextCommitted_Lambda([this](const FText&, ETextCommit::Type CommitType)
							{
								if (CommitType == ETextCommit::OnEnter)
								{
									OnSave();
								}
							})
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("SSaveMaskWindow_SaveButton", "Save"))
						.OnClicked(this, &SSaveMaskWindow::OnSave)
					]
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("SSaveMaskWindow_SkipSavingButton", "Skip Saving"))
						.OnClicked(this, &SSaveMaskWindow::OnSkip)
					]
				]
			];
		}

		bool SupportsKeyboardFocus() const override { return true; }

		bool IsNameValid()
		{
			return !ExistingNames.Contains(FName(*ResultPtr->AttributeName.ToString()));
		}
		FReply OnSave()
		{
			if (IsNameValid())
			{
				ResultPtr->bSave = true;
				CloseWindow();
			}
			
			return FReply::Handled();
		}

		FReply OnSkip()
		{
			CloseWindow();
			return FReply::Handled();
		}

		void CloseWindow()
		{
			if ( WidgetWindow.IsValid() )
			{
				WidgetWindow.Pin()->RequestDestroyWindow();
			}	
		}

		TWeakPtr<SWindow> WidgetWindow;
		TArray<FName> ExistingNames;

		FResult* ResultPtr = nullptr;
	};
	

}

void UMorphTargetMaskTool::CommitToolMesh(FDynamicMesh3& InToolMesh)
{
	using namespace UE::Geometry;

	if (MaskToolProperties->ShouldSaveMask())
	{
		FDynamicMeshWeightAttribute* TargetWeightMap = nullptr;
		if (!MaskToolProperties->ShouldSaveNewMask())
		{
			TargetWeightMap = MaskToolCommitMesh.Attributes()->GetWeightLayer(CurrentAttributeIndex);
		}
		else
		{
			/** Create the window to host our package dialog widget */
			TSharedRef<SWindow> SaveMaskModalWindow = SNew(SWindow)
				.Title(FText::FromString("Save Vertex Attribute"))
				.SizingRule(ESizingRule::Autosized);

			
			using namespace UE::MorphTargetMaskTool::Private;
			SSaveMaskWindow::FResult Result;
			FName InitialName = FName(GetBaseMorphTarget().ToString() + TEXT("_Mask"));

			while (MaskToolProperties->VertexAttributes.Contains(InitialName))
			{
				InitialName.SetNumber(InitialName.GetNumber() + 1);
			}
			
			Result.AttributeName = FText::FromName(InitialName);
			
			/** Set the content of the window to our package dialog widget */
			TSharedRef<SSaveMaskWindow> SaveMaskWindowWidget=
				SNew(SSaveMaskWindow)
				.WidgetWindow(SaveMaskModalWindow)
				.ExistingNames(MaskToolProperties->VertexAttributes)
				.ResultPtr(&Result);

			SaveMaskModalWindow->SetContent(SaveMaskWindowWidget);

			/** Show the dialog window as a modal window */
			GEditor->EditorAddModalWindow(SaveMaskModalWindow);

			if (Result.bSave)
			{
				// Create a new weight layer to save into
				const int32 NumAttributeLayers = MaskToolCommitMesh.Attributes()->NumWeightLayers();
				MaskToolCommitMesh.Attributes()->SetNumWeightLayers(NumAttributeLayers + 1);
				TargetWeightMap = MaskToolCommitMesh.Attributes()->GetWeightLayer(NumAttributeLayers);
				TargetWeightMap->SetName(*Result.AttributeName.ToString());

				// Override the chosen option now that we know the name of the new attribute
				MaskToolProperties->VertexAttributeOption = TargetWeightMap->GetName();
			}
		}

		if (TargetWeightMap)
		{
			FDynamicMeshWeightAttribute* SourceWeightMap = InToolMesh.Attributes()->GetWeightLayer(CurrentAttributeIndex);

			if (IsolationSubmesh)
			{
				// Iterate submesh vertices, remap to full mesh
				ParallelFor(InToolMesh.MaxVertexID(), [this, SourceWeightMap, TargetWeightMap, &InToolMesh](int32 SubVID)
				{
					if (!InToolMesh.IsVertex(SubVID))
					{
						return;
					}

					int32 FullVID = IsolationSubmesh->MapVertexToBaseMesh(SubVID);
					if (FullVID != FDynamicMesh3::InvalidID && MaskToolCommitMesh.IsVertex(FullVID))
					{
						float Weight = 0.0f;
						SourceWeightMap->GetValue(SubVID, &Weight);
						TargetWeightMap->SetValue(FullVID, &Weight);
					}
				});
			}
			else
			{
				ParallelFor(MaskToolCommitMesh.MaxVertexID(), [this, SourceWeightMap, TargetWeightMap](int32 VertID)
				{
					if (!MaskToolCommitMesh.IsVertex(VertID))
					{
						return;
					}
				
					float Weight = 0.0f;
					SourceWeightMap->GetValue(VertID, &Weight);
					TargetWeightMap->SetValue(VertID, &Weight);
				});
			}
		}
	}

	MaskToolCommitMesh.Attributes()->RemoveMorphTargetAttribute(ToolMorphTargetBackupAttributeName);
	
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->BeginUndoTransaction(LOCTEXT("MorphTargetMaskTool_TransactionName", "Mask Morph Target"));

		// commit the mask tool mesh cache instead of the sculpt mesh/base tool tool mesh
		// setting ModifiedTopology to true to force a full dyna mesh update such that weight layer changes
		// are converted to mesh description as well;
		// todo: update weight layers only
		constexpr bool bForceModifiedTopology = true;
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, MaskToolCommitMesh, bForceModifiedTopology);
		
		ToolManager->EndUndoTransaction();
	}

	if (EditorContext.IsValid())
	{
		EditorContext->NotifyMorphTargetEdited();
	}
}

void UMorphTargetMaskTool::UpdatePreview(const TSet<int32>* TrianglesToUpdate, const TArray<int32>* VerticesToUpdate)
{
	Super::UpdatePreview(TrianglesToUpdate, VerticesToUpdate);

	using namespace UE::Geometry;
	
	FDynamicMeshMorphTargetAttribute* ToolMorphTargetCache = MaskToolCommitMesh.Attributes()->GetMorphTargetAttribute(ToolMorphTargetName);

	FName BaseMorphTargetAttributeName = GetBaseMorphTarget() == ToolMorphTargetName ? ToolMorphTargetBackupAttributeName : GetBaseMorphTarget();  
	FDynamicMeshMorphTargetAttribute* BaseToolMorphTarget = MaskToolCommitMesh.Attributes()->GetMorphTargetAttribute(BaseMorphTargetAttributeName);

	int32 NumVerts = VerticesToUpdate ? VerticesToUpdate->Num() : GetSculptMesh()->MaxVertexID();
	
	ParallelFor(NumVerts, [VerticesToUpdate, this, ToolMorphTargetCache, BaseToolMorphTarget](int32 ThreadID)
	{
		int32 SubVID = VerticesToUpdate
			? (*VerticesToUpdate)[ThreadID]
			: ThreadID;

		// Remap submesh VID to full mesh VID for MaskToolCommitMesh access
		int32 FullVID = IsolationSubmesh ? IsolationSubmesh->MapVertexToBaseMesh(SubVID) : SubVID;

		if (FullVID == FDynamicMesh3::InvalidID || !MaskToolCommitMesh.IsVertex(FullVID))
		{
			return;
		}

		const float Weight = VertexData.GetValue(SubVID);

		FVector Delta;
		BaseToolMorphTarget->GetValue(FullVID, Delta);

		Delta *= (1.0f - Weight);
		ToolMorphTargetCache->SetValue(FullVID, Delta);
	});

	const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
	const TMap<FName, float>& MorphTargetWeights = GetMorphTargetWeights();
	PosePreviewMesh(ComponentSpaceTransforms, MorphTargetWeights, TrianglesToUpdate, VerticesToUpdate);
	
	if (TrianglesToUpdate)
	{
		Octree.ReinsertTriangles(*TrianglesToUpdate);
	}
	else
	{
		bRebuildOctree = true;
	}

	bMeshSelectorNeedsUpdate = true;
}

FName UMorphTargetMaskTool::GetEditingMorphTarget()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetEditingMorphTarget();
	}

	return {};	
}

FName UMorphTargetMaskTool::GetBaseMorphTarget()
{
	return MaskToolProperties->BaseMorphTarget;
}

TMap<FName, float> UMorphTargetMaskTool::GetMorphTargetWeights()
{
	// Brush mirroring only suppresses pose while painting in brush mode; in mesh-selection mode
	// the mirror plane is irrelevant and the user expects pose to follow the editor context.
	if (IsBrushMirroringEnabled() && IsInBrushMode())
	{
		return {};
	}

	if (EditorContext.IsValid())
	{
		return EditorContext->GetMorphTargetWeights();
	}

	return {};
}

const TArray<FTransform>& UMorphTargetMaskTool::GetComponentSpaceBoneTransforms()
{
	if (IsBrushMirroringEnabled() && IsInBrushMode())
	{
		return ComponentSpaceTransformsRefPose;
	}
	
	if (EditorContext.IsValid())
	{
		return EditorContext->GetComponentSpaceBoneTransforms(GetTarget());
	}

	if (DefaultComponentSpaceBoneTransforms.IsEmpty())
	{
		ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(GetTarget());
		SkeletonProvider->GetSkeleton().GetBoneAbsoluteTransforms(DefaultComponentSpaceBoneTransforms);
	}
	
	return DefaultComponentSpaceBoneTransforms;
}

void UMorphTargetMaskTool::ToggleBoneManipulation(bool bEnable)
{
	if (EditorContext.IsValid())
	{
		EditorContext->ToggleBoneManipulation(bEnable);
	}
}

void UMorphTargetMaskTool::HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload)
{
	if (InStroke())
	{
		return;
	}
	
	using namespace SkeletalMeshToolsHelper;
	if (Payload.CurrentState == FPoseChangeDetector::PoseStoppedChanging)
	{
		bRebuildOctree = true;
	}
	else if (Payload.CurrentState == FPoseChangeDetector::PoseJustChanged ||
		Payload.CurrentState == FPoseChangeDetector::PoseChanged)
	{
		bFastDeformPreviewMesh = true;
		bMeshSelectorNeedsUpdate = true;
	}
}



void UMorphTargetMaskTool::Setup()
{
	EditorContext = GetToolManager()->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();
	SetupMorphEditingToolCommon();
	
	ToolMorphTargetName = GetEditingMorphTarget();



	// Add the mask tool settings to the top. Property init takes place during SetupToolMesh()
	MaskToolProperties = NewObject<UMorphTargetMaskToolProperties>(this);
	AddToolPropertySource(MaskToolProperties);
	


	
	const FReferenceSkeleton& RefSkeleton = CastChecked<ISkeletonProvider>(GetTarget())->GetSkeleton();
	RefSkeleton.GetBoneAbsoluteTransforms(ComponentSpaceTransformsRefPose);
	
	MorphTargetZeroWeights = GetMorphTargetWeights();
	for (TPair<FName, float>& MorphInfo : MorphTargetZeroWeights)
	{
		MorphInfo.Value = 0.0f;
	}

	ToggleBoneManipulation(true);

	PoseChangeDetector.GetNotifier().AddUObject(this, &UMorphTargetMaskTool::HandlePoseChangeDetectorEvent);
	
	// Setup Vertex Sculpt Tool
	Super::Setup();


	BaseMorphTargetWatcher.Initialize([this]()
	{
		return GetBaseMorphTarget();
	},
[this](const FName& InNewBaseMorphTarget)
	{
		UpdatePreview();
	},
GetBaseMorphTarget());

	
	VertexAttributeOptionWatcher.Initialize([this]()
	{
		return MaskToolProperties->VertexAttributeOption;
	},
[this](const FName& InNewVertexAttributeOption)
	{
		if (InNewVertexAttributeOption != MaskToolProperties->CreateNewVertexAttributeOption && 
			InNewVertexAttributeOption != MaskToolProperties->DoNotSaveVertexAttributeOption)
		{
			// Editing a specific attribute
			int32 WeightMapIndex = MaskToolProperties->VertexAttributes.IndexOfByKey(InNewVertexAttributeOption);
			SetAttributeToPaint(WeightMapIndex);
			CurrentAttributeIndex = WeightMapIndex;
		}
		else
		{
			if (CurrentAttributeIndex != TempPaintLayerAttributeIndex)
			{
				SetAttributeToPaint(TempPaintLayerAttributeIndex);
				CurrentAttributeIndex = TempPaintLayerAttributeIndex;
			}
		}
	},
	MaskToolProperties->VertexAttributeOption);
	
	// Hard to do real change detection for masking since 
	// it requires iterating the mesh verts 
	bAnyChangeMade = true;

	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(GetSculptMeshComponent()));
}



void UMorphTargetMaskTool::Shutdown(EToolShutdownType ShutdownType)
{
	Super::Shutdown(ShutdownType);

	MaskToolProperties->Shutdown(this, ShutdownType);
	
	ShutdownMorphEditingToolCommon();	
}

void UMorphTargetMaskTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (!InStroke())
	{
		BaseMorphTargetWatcher.CheckAndUpdate();
		VertexAttributeOptionWatcher.CheckAndUpdate();
		
		const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
		TMap<FName, float> MorphTargetWeights = GetMorphTargetWeights();
		
		PoseChangeDetector.CheckPose(ComponentSpaceTransforms, MorphTargetWeights);

		if (bFastDeformPreviewMesh)
		{
			bFastDeformPreviewMesh = false;
			PosePreviewMesh(ComponentSpaceTransforms, MorphTargetWeights);
		}

		if (GetEditingMode() == EMeshVertexAttributePaintToolEditMode::Mesh && bMeshSelectorNeedsUpdate)
		{
			bMeshSelectorNeedsUpdate = false;
			MeshSelector->UpdateAfterMeshDeformation();	
		}

		if (bRebuildOctree)
		{
			bRebuildOctree = false;

			RebuildOctree();
		}
	}
}

bool UMorphTargetMaskTool::IsInputIsolationValidOnOutput() const
{
	return true;
}

void UMorphTargetMaskTool::GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const
{
	OutHints.Add({ LOCTEXT("HintBrushSize", "Brush Size"), LOCTEXT("HintBrushSizeChord", "B + drag left/right") });
	OutHints.Add({ LOCTEXT("HintAttribValue", "Attribute Value"), LOCTEXT("HintAttribValueChord", "B + drag up/down") });
}

void UMorphTargetMaskTool::SetupCommonProperties(const TFunction<void(UMorphTargetEditingToolProperties*)>& InSetupFunction)
{
	EditorToolProperties = NewObject<UMorphTargetEditingToolProperties>(this);

	InSetupFunction(EditorToolProperties);
	
	AddToolPropertySource(EditorToolProperties);
}

void UMorphTargetMaskTool::PosePreviewMesh(const TArray<FTransform>& ComponentSpaceTransforms, const TMap<FName, float>& MorphTargetWeights, const TSet<int32>* TrianglesToUpdate, const TArray<int32>* VerticesToUpdate)
{
	using namespace SkeletalMeshToolsHelper;
	
	TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(ComponentSpaceTransformsRefPose, ComponentSpaceTransforms);

	FDynamicMesh3& SculptMesh = *GetSculptMesh();

	// When isolated, pose only the isolated vertices and remap IDs to submesh space.
	// GetPosedMesh operates on MaskToolCommitMesh (full mesh), so we pass full-mesh vertex IDs.
	TArray<int32> FullMeshVertArrayForIsolation;
	if (IsolationSubmesh)
	{
		if (VerticesToUpdate)
		{
			FullMeshVertArrayForIsolation.Reserve(VerticesToUpdate->Num());
			for (int32 SubVID : *VerticesToUpdate)
			{
				FullMeshVertArrayForIsolation.Add(IsolationSubmesh->MapVertexToBaseMesh(SubVID));
			}
		}
		else
		{
			FullMeshVertArrayForIsolation.Reserve(IsolationSubmesh->GetSubmesh().VertexCount());
			for (int32 SubVID : IsolationSubmesh->GetSubmesh().VertexIndicesItr())
			{
				FullMeshVertArrayForIsolation.Add(IsolationSubmesh->MapVertexToBaseMesh(SubVID));
			}
		}
	}
	
	const TArray<int32>* FinalFullMeshVertArrayPtr = nullptr;
	if (IsolationSubmesh)
	{
		FinalFullMeshVertArrayPtr = &FullMeshVertArrayForIsolation;
	}
	else
	{
		FinalFullMeshVertArrayPtr = VerticesToUpdate;
	}

	auto WriteFunc = [&](FVertInfo VertInfo, const FVector& PosedVertPos)
		{
			int32 SculptVID = IsolationSubmesh ? IsolationSubmesh->MapVertexToSubmesh(VertInfo.VertID) : VertInfo.VertID;
			if (SculptVID != FDynamicMesh3::InvalidID)
			{
				SculptMesh.SetVertex(SculptVID, PosedVertPos);
			}
		};

	GetPosedMesh(WriteFunc, MaskToolCommitMesh, BoneMatrices, NAME_None, MorphTargetWeights, FinalFullMeshVertArrayPtr ? *FinalFullMeshVertArrayPtr : TArray<int32>());
	if (!TrianglesToUpdate)
	{
		FMeshNormals::QuickRecomputeOverlayNormals(SculptMesh);
	}
	else
	{
		FMeshNormals::RecomputeOverlayTriNormals(SculptMesh, TrianglesToUpdate->Array());
	}

	constexpr bool bNormal = true;
	CastChecked<UDynamicMeshComponent>(GetSculptMeshComponent())->FastNotifyPositionsUpdated(bNormal);

	MeshElementsDisplay->NotifyMeshChanged();
}


#undef LOCTEXT_NAMESPACE

