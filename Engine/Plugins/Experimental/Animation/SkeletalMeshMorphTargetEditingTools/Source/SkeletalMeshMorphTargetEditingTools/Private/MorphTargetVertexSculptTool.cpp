// Copyright Epic Games, Inc. All Rights Reserved.

#include "MorphTargetVertexSculptTool.h"

#include "ContextObjectStore.h"
#include "ToolBuilderUtil.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolChange.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"
#include "StaticMeshAttributes.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "IPersonaEditorModeManager.h"
#include "PersonaModule.h"
#include "SkeletalMeshOperations.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "AnimationRuntime.h"
#include "EraseMorphTargetBrushOps.h"
#include "PreviewMesh.h"
#include "SkeletalMeshAttributes.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "Components/DynamicMeshComponent.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "DynamicMesh/MeshNormals.h"
#include "Sculpting/MeshInflateBrushOps.h"
#include "Sculpting/MeshMoveBrushOps.h"
#include "Sculpting/MeshPlaneBrushOps.h"
#include "Sculpting/MeshSculptBrushOps.h"
#include "Sculpting/MeshSmoothingBrushOps.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "ToolSetupUtil.h"
#include "Parameterization/MeshPlanarSymmetry.h"
#include "InteractiveToolActionSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MorphTargetVertexSculptTool)

#define LOCTEXT_NAMESPACE "MorphTargetVertexSculptTool"


bool UMorphTargetVertexSculptToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	USkeletalMeshEditorContextObjectBase* EditorContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();
	if (!EditorContext)
	{
		return false;
	}

	if (EditorContext->GetEditingMorphTarget() == NAME_None)
	{
		return false;
	}
	
	return Super::CanBuildTool(SceneState);
}

UMeshSurfacePointTool* UMorphTargetVertexSculptToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMorphTargetVertexSculptTool* MorphTargetEditorTool = NewObject<UMorphTargetVertexSculptTool>(SceneState.ToolManager);
	MorphTargetEditorTool->SetWorld(SceneState.World);
	return MorphTargetEditorTool;
}

const FToolTargetTypeRequirements& UMorphTargetVertexSculptToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements ToolRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		USceneComponentBackedTarget::StaticClass(),
		USkeletonProvider::StaticClass(),
	});

	return ToolRequirements;
}

FName UMorphTargetVertexSculptTool::GetEditingMorphTarget()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetEditingMorphTarget();
	}

	return {};	
}

TMap<FName, float> UMorphTargetVertexSculptTool::GetMorphTargetWeights()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetMorphTargetWeights();
	}

	return {};	
}

const TArray<FTransform>& UMorphTargetVertexSculptTool::GetComponentSpaceBoneTransforms()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetComponentSpaceBoneTransforms(GetTarget());
	}

	if (DefaultComponentSpaceBoneTransforms.IsEmpty())
	{
		ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(GetTarget());
		SkeletonProvider->GetSkeleton().GetBoneAbsoluteTransforms(DefaultComponentSpaceBoneTransforms);
	}
	
	return DefaultComponentSpaceBoneTransforms;
}

void UMorphTargetVertexSculptTool::ToggleBoneManipulation(bool bEnable)
{
	if (EditorContext.IsValid())
	{
		EditorContext->ToggleBoneManipulation(bEnable);
	}
}

void UMorphTargetVertexSculptTool::HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload)
{
	if (!AllowToolMeshUpdates())
	{
		return;
	}
	
	using namespace SkeletalMeshToolsHelper;
	if (Payload.CurrentState == FPoseChangeDetector::PoseStoppedChanging)
	{
		bFullRefreshSculptMesh = true;
	}
	else if (Payload.CurrentState == FPoseChangeDetector::PoseJustChanged ||
		Payload.CurrentState == FPoseChangeDetector::PoseChanged)
	{
		bFastDeformSculptMesh = true;
	}
}



void UMorphTargetVertexSculptTool::InitializeSculptMeshComponent(UBaseDynamicMeshComponent* Component, AActor* Actor)
{
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Component, nullptr);
	Component->SetShadowsEnabled(false);
	Component->SetupAttachment(Actor->GetRootComponent());
	Component->RegisterComponent();

	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	FDynamicMesh3 FullMesh = UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams);

	ToolDynamicMesh = NewObject<UDynamicMesh>();
	ToolDynamicMesh->SetMesh(MoveTemp(FullMesh));

	// Build isolated submesh if isolation was active, set the correct mesh on the preview/sculpt component
	ToolDynamicMesh->ProcessMesh([&](const FDynamicMesh3& Mesh)
		{
			const FDynamicMesh3* MeshForPreview = &Mesh;
			
			if (EditorContext.IsValid())
			{
				const TArray<int32>& IsolatedTriangles = EditorContext->GetIsolatedTriangles();
				if (!IsolatedTriangles.IsEmpty())
				{
					IsolationSubmesh = UE::Geometry::FDynamicSubmesh3(&Mesh, IsolatedTriangles);
					MeshForPreview = &IsolationSubmesh->GetSubmesh();
				}
			}

			Component->SetMesh(FDynamicMesh3(*MeshForPreview));
		});


	// Bake rotation and scaling into the component's preview mesh only so that we don't have to deal with skewed brush stamps
	// TODO: for this to work for morph targets we will need to unbake the transforms before extracting the morph delta, so comment out this part for now
	
	// InitialTargetTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	// InitialTargetTransform.ClampMinimumScale(0.01);
	// FVector3d Translation = InitialTargetTransform.GetTranslation();
	// InitialTargetTransform.SetTranslation(FVector3d::Zero());
	// Component->ApplyTransform(InitialTargetTransform, false);
	// CurTargetTransform = UE::Geometry::FTransformSRT3d(Translation);
	// Component->SetWorldTransform((FTransform)CurTargetTransform);

	UE::ToolTarget::HideSourceObject(Target);
}

void UMorphTargetVertexSculptTool::Setup()
{
	EditorContext = GetToolManager()->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();
	
	SetupMorphEditingToolCommon();

	// Allow brush radius to shrink small enough to affect roughly a single vertex on dense morph target meshes.
	// Base class default is 0.01 which is too coarse for face sculpting at typical mesh scales.
	MinimumBrushAdaptiveSizeRatio = 0.001;

	// Setup Vertex Sculpt Tool
	Super::Setup();

	ViewProperties->MaterialMode = EMeshEditingMaterialModes::ExistingMaterial;

	// ToolDynamicMesh created in InitializeSculptMeshComponent
	
	// Remove unused attributes to make undo/redo faster
	GetSculptMesh()->Attributes()->RemoveAllMorphTargetAttributes();
	GetSculptMesh()->Attributes()->RemoveAllSkinWeightsAttributes();
	
	constexpr bool bCopyNormals = false, bCopyColors = false, bCopyUVs = false, bCopyAttributes = false;
	PosedMeshWithoutEditingMorph.Copy(*GetSculptMesh(), bCopyNormals, bCopyColors, bCopyUVs, bCopyAttributes);
	
	ToolMorphTargetName = GetEditingMorphTarget();
	
	const FReferenceSkeleton& RefSkeleton = CastChecked<ISkeletonProvider>(GetTarget())->GetSkeleton();
	RefSkeleton.GetBoneAbsoluteTransforms(ComponentSpaceTransformsRefPose);

	ToggleBoneManipulation(true);

	PoseChangeDetector.GetNotifier().AddUObject(this, &UMorphTargetVertexSculptTool::HandlePoseChangeDetectorEvent);

	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(GetSculptMeshComponent()));
}

namespace MorphTargetVertexSculptTool::Private
{
	// Subclass FSmoothBrushOp / FFlattenBrushOp / FInflateBrushOp to also support Spacing-mode stroke
	// emission so the morph target tool's Smooth/Flatten/Inflate brushes stop accumulating while the
	// cursor is stationary. The upstream base default rejects Spacing, which drops
	// UpdateStampPendingState into a per-tick emit fallback.
	class FMorphTargetSmoothBrushOp : public FSmoothBrushOp
	{
	public:
		virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
		{
			return StrokeType == EMeshSculptStrokeType::Spacing
				|| FSmoothBrushOp::SupportsStrokeType(StrokeType);
		}
	};

	class FMorphTargetFlattenBrushOp : public FFlattenBrushOp
	{
	public:
		virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
		{
			return StrokeType == EMeshSculptStrokeType::Spacing
				|| FFlattenBrushOp::SupportsStrokeType(StrokeType);
		}
	};

	class FMorphTargetInflateBrushOp : public FInflateBrushOp
	{
	public:
		virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
		{
			return StrokeType == EMeshSculptStrokeType::Spacing
				|| FInflateBrushOp::SupportsStrokeType(StrokeType);
		}
	};
}

void UMorphTargetVertexSculptTool::SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior)
{
	Super::SetupBrushEditBehaviorSetup(OutBehavior);

	// Make Adaptive-mode scrub match World-mode's screen-space behavior: 1 cursor pixel = constant on-screen
	// radius pixels
	OutBehavior.HorizontalProperty.MutateDeltaFunc = [this](float Delta) -> float
	{
		const float CameraDistance = static_cast<float>((CameraState.Position - LastBrushFrameWorld.Origin).Length());
		if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
		{
			const float RangeSize = BrushProperties->BrushSize.WorldSizeRange.Size();
			return RangeSize > KINDA_SMALL_NUMBER ? Delta * CameraDistance / RangeSize : Delta;
		}
		return Delta * CameraDistance;
	};
}

void UMorphTargetVertexSculptTool::RegisterBrushes()
{
	// Curated brush list for face morph target sculpting. Skip Super::RegisterBrushes() entirely
	// and register only the brushes we want, so the Kelvinlet, Sculpt*, SmoothFill, and Plane
	// variants from the base do not appear in the dropdown. If upstream adds new brushes to
	// UMeshVertexSculptTool::RegisterBrushes(), they need to be opted-in here explicitly.

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Move, LOCTEXT("Move", "Move"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FMoveBrushOp>>(),
		NewObject<UMoveBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Offset, LOCTEXT("Offset", "SculptN"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<UE::Geometry::FSingleNormalSculptBrushOp>>(),
		NewObject<UStandardSculptBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Inflate, LOCTEXT("Inflate", "Inflate"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<MorphTargetVertexSculptTool::Private::FMorphTargetInflateBrushOp>>(),
		NewObject<UInflateBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Flatten, LOCTEXT("Flatten", "Flatten"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<MorphTargetVertexSculptTool::Private::FMorphTargetFlattenBrushOp>>(),
		NewObject<UFlattenBrushOpProps>(this));

	// Had to hijack the EraseSculptLayer identifier from base mesh vertex sculpt tool for our erase
	// morph target tool since it is the simplest way to get an icon for the tool.
	RegisterBrushType(
		(int32)EMeshVertexSculptBrushType::EraseSculptLayer,
		LOCTEXT("EraseSculptLayerBrushTypeName", "EraseSculptLayer"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FEraseMorphTargetBrushOp>(GetMeshWithoutCurrentMorphFunc);}),
		NewObject<UEraseMorphTargetBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Smooth, LOCTEXT("SmoothBrush", "Smooth"),
	MakeUnique<TBasicMeshSculptBrushOpFactory<MorphTargetVertexSculptTool::Private::FMorphTargetSmoothBrushOp>>(),
	NewObject<USmoothBrushOpProps>(this));
	
	// Secondary brush (shift-modifier smoothing). Activated as ID 0; matches base behaviour.
	RegisterSecondaryBrushType(0, LOCTEXT("Smooth", "Smooth"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<MorphTargetVertexSculptTool::Private::FMorphTargetSmoothBrushOp>>(),
		NewObject<USecondarySmoothBrushOpProps>(this));

	GetMeshWithoutCurrentMorphFunc = [this]()
	{
		if (bPosedMeshWithoutEditingMorphDirty)
		{
			UpdatePosedMeshWithoutEditingMorph();
		}
		return &PosedMeshWithoutEditingMorph;
	};



	// Override a few brush-op-property defaults. These run before OnCompleteSetup() calls
	// RestoreAllBrushTypeProperties; that restore is a no-op when no cache entry exists for a
	// brush, so the overrides act as one-time defaults for fresh sessions while in-session user
	// adjustments still persist (CachedPropertiesMap is transient and resets on editor restart,
	// restoring these defaults again).

	// Move brush: default Falloff to 1.0 (UMoveBrushOpProps default is 0.5).
	if (TObjectPtr<UMeshSculptBrushOpProps>* Props = BrushOpPropSets.Find((int32)EMeshVertexSculptBrushType::Move))
	{
		if (UMoveBrushOpProps* MoveProps = Cast<UMoveBrushOpProps>(*Props))
		{
			MoveProps->Falloff = 1.0f;
		}
	}

	// Smooth brushes (primary + secondary): favor PolyGroup edges by default to better preserve
	// edge flow during morph target sculpting. Per the UPROPERTY comment in MeshSmoothingBrushOps.h:
	// "No effect where PolyGroups are not defined." On vertices not on a polygroup boundary the
	// branch is a no-op (FoundGroupEdges == 0).
	if (TObjectPtr<UMeshSculptBrushOpProps>* Props = BrushOpPropSets.Find((int32)EMeshVertexSculptBrushType::Smooth))
	{
		if (USmoothBrushOpProps* SmoothProps = Cast<USmoothBrushOpProps>(*Props))
		{
			SmoothProps->bFavorPolyGroupEdges = true;
		}
	}
	if (TObjectPtr<UMeshSculptBrushOpProps>* Props = SecondaryBrushOpPropSets.Find(0))
	{
		if (USecondarySmoothBrushOpProps* SmoothProps = Cast<USecondarySmoothBrushOpProps>(*Props))
		{
			SmoothProps->bFavorPolyGroupEdges = true;
		}
	}
}

void UMorphTargetVertexSculptTool::Shutdown(EToolShutdownType ShutdownType)
{
	Super::Shutdown(ShutdownType);

	ShutdownMorphEditingToolCommon();	
}

void UMorphTargetVertexSculptTool::CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("MorphTargetSculptMeshToolTransactionName", "Sculpt Morph Target"));

	ToolDynamicMesh->ProcessMesh([&](const FDynamicMesh3& InMesh)
		{
			// Commit the tool mesh cache instead of the sculpt mesh
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, InMesh, bModifiedTopology);
		});

	if (EditorContext.IsValid())
	{
		EditorContext->NotifyMorphTargetEdited();
	}
	
	GetToolManager()->EndUndoTransaction();	
}

void UMorphTargetVertexSculptTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (!InStroke())
	{
		const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
		const TMap<FName, float>& MorphTargetWeights = GetMorphTargetWeights();
		
		PoseChangeDetector.CheckPose(ComponentSpaceTransforms, MorphTargetWeights);

		if (bFastDeformSculptMesh)
		{
			bFastDeformSculptMesh = false;
			
			PoseSculptMesh(ComponentSpaceTransforms, MorphTargetWeights);
		}

		if (bFullRefreshSculptMesh)
		{
			bFullRefreshSculptMesh = false;
			
			// Using a dummy mesh replacement change to trigger octree/normals/base mesh refresh
			TSharedPtr<FDynamicMesh3> DummySculptMesh = MakeShared<FDynamicMesh3>(*GetSculptMesh());	
			FMeshReplacementChange DummyChange(DummySculptMesh, DummySculptMesh);
			OnDynamicMeshComponentChanged(DynamicMeshComponent, &DummyChange, false);
		}
	}

	// Hide the brush alpha mesh when alpha is not used because it makes it hard to see the sculpt area clearly.
	BrushIndicatorMesh->SetVisible(AlphaProperties->IsPropertySetEnabled() && AlphaProperties->Alpha != nullptr);
}

void UMorphTargetVertexSculptTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);
}


/*
 * internal Change classes
 */

class FMorphTargetVertexSculptNonSymmetricChange : public FToolCommandChange
{
public:
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
};


void FMorphTargetVertexSculptNonSymmetricChange::Apply(UObject* Object)
{
	if (Cast<UMorphTargetVertexSculptTool>(Object))
	{
		Cast<UMorphTargetVertexSculptTool>(Object)->UndoRedo_RestoreSymmetryPossibleState(false);
	}
}
void FMorphTargetVertexSculptNonSymmetricChange::Revert(UObject* Object)
{
	if (Cast<UMorphTargetVertexSculptTool>(Object))
	{
		Cast<UMorphTargetVertexSculptTool>(Object)->UndoRedo_RestoreSymmetryPossibleState(true);
	}
}

void UMorphTargetVertexSculptTool::OnBeginStroke(const FRay& WorldRay)
{
	Super::OnBeginStroke(WorldRay);
}

using FMeshMorphTargetChange = SkeletalMeshToolsHelper::FMeshMorphTargetChange;



void UMorphTargetVertexSculptTool::OnEndStroke()
{
	// update spatial
	bTargetDirty = true;

	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	check(ActiveVertexChange);

	TMap<FName, float> MorphTargetWeights = MorphTargetWeightsForPosedMesh;
	const TArray<FTransform>& ComponentSpaceTransforms = ComponentSpaceTransformsForPosedMesh; 

	float EditingMorphTargetWeight = MorphTargetWeights[ToolMorphTargetName];
	
	// Don't update the cache if the morph cannot be extracted from the mesh
	if (FMath::IsNearlyZero(EditingMorphTargetWeight))
	{
		bFastDeformSculptMesh = true;
		bFullRefreshSculptMesh = true;
	}
	else
	{
		// Extract morph delta if possible
		if (!ActiveVertexChange->Change->Vertices.IsEmpty())
		{
			FMeshMorphTargetChange* MorphTargetChange = new FMeshMorphTargetChange();
			MorphTargetChange->MorphTargetName = ToolMorphTargetName;
			MorphTargetChange->Vertices = MoveTemp(ActiveVertexChange->Change->Vertices);
			MorphTargetChange->OldDeltas.SetNumUninitialized(MorphTargetChange->Vertices.Num());
			MorphTargetChange->NewDeltas.SetNumUninitialized(MorphTargetChange->Vertices.Num());

			// Exclude the morph target we are trying to extract
			MorphTargetWeights.Remove(ToolMorphTargetName);

			using namespace SkeletalMeshToolsHelper;
			TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(
				ComponentSpaceTransformsRefPose,
				ComponentSpaceTransforms
			);

			ToolDynamicMesh->EditMesh([&](FDynamicMesh3& Mesh)
				{
					FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = Mesh.Attributes()->GetMorphTargetAttribute(ToolMorphTargetName);

					// MorphTargetChange records base VIDs and the GetUnposedMesh also
					// iterates base VIDs directly
					if (IsolationSubmesh)
					{
						for (int32& VID : MorphTargetChange->Vertices)
						{
							VID = IsolationSubmesh->MapVertexToBaseMesh(VID);
						}
					}

					auto WriteFunction = [&](FVertInfo VertInfo, const FVector& UnposedVertPos)
						{
							const int32 FullVID = VertInfo.VertID;
							if (!Mesh.IsVertex(FullVID))
							{
								return;
							}

							FVector NewDelta = UnposedVertPos - Mesh.GetVertexRef(FullVID);
							NewDelta /= EditingMorphTargetWeight;

							FVector OldDelta;
							MorphTargetAttribute->GetValue(FullVID, OldDelta);
							MorphTargetAttribute->SetValue(FullVID, NewDelta);

							MorphTargetChange->OldDeltas[VertInfo.VertArrayIndex] = OldDelta;
							MorphTargetChange->NewDeltas[VertInfo.VertArrayIndex] = NewDelta;
						};

					const FDynamicMesh3& SculptMesh = *GetSculptMesh();
					const UE::Geometry::FDynamicSubmesh3* SubmeshPtr = IsolationSubmesh.GetPtrOrNull();

					auto GetPosedVertexFunc = [&SculptMesh, SubmeshPtr](int32 BaseVID) -> FVector
					{
						const int32 PosedVID = SubmeshPtr ? SubmeshPtr->MapVertexToSubmesh(BaseVID) : BaseVID;
						return SculptMesh.GetVertex(PosedVID);
					};

					GetUnposedMesh(WriteFunction, GetPosedVertexFunc, Mesh, BoneMatrices, NAME_None, MorphTargetWeights, MorphTargetChange->Vertices);
				});


			TUniquePtr<TWrappedToolCommandChange<FMeshMorphTargetChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshMorphTargetChange>>();

			NewChange->WrappedChange.Reset(MorphTargetChange);
			NewChange->BeforeModify = [this](bool bRevert)
				{
					WaitForPendingUndoRedoUpdate();
				};


			NewChange->AfterModify = [this](bool bRevert)
				{
					bFastDeformSculptMesh = true;
					bFullRefreshSculptMesh = true;
				};

			GetToolManager()->EmitObjectChange(ToolDynamicMesh, MoveTemp(NewChange), LOCTEXT("VertexSculptChange", "Brush Stroke"));
			if (bMeshSymmetryIsValid && bApplySymmetry == false)
			{
				// if we end a stroke while symmetry is possible but disabled, we now have to assume that symmetry is no longer possible
				GetToolManager()->EmitObjectChange(this, MakeUnique<FMorphTargetVertexSculptNonSymmetricChange>(),
					LOCTEXT("DisableSymmetryChange", "Disable Symmetry"));
				bMeshSymmetryIsValid = false;
				SymmetryProperties->bSymmetryCanBeEnabled = bMeshSymmetryIsValid;
			}
		}
	}
	
	LongTransactions.Close(GetToolManager());

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
}

bool UMorphTargetVertexSculptTool::CanMirrorEditingMorphTarget() const
{
	if (InStroke())
	{
		return false;
	}
	if (!EditorContext.IsValid())
	{
		return false;
	}
	if (ToolMorphTargetName == NAME_None)
	{
		return false;
	}
	return EditorContext->GetBaseMeshSymmetry() != nullptr;
}

void UMorphTargetVertexSculptTool::MirrorEditingMorphTarget()
{
	if (!CanMirrorEditingMorphTarget())
	{
		return;
	}

	WaitForPendingStampUpdateConst();
	WaitForPendingUndoRedoUpdate();

	// Local name avoids hiding UMeshVertexSculptTool::Symmetry (the brush-symmetry pimpl member).
	const UE::Geometry::FMeshPlanarSymmetry* BaseSymmetry = EditorContext->GetBaseMeshSymmetry();
	if (!BaseSymmetry)
	{
		return;
	}

	FMeshMorphTargetChange* MorphChange = new FMeshMorphTargetChange();
	ToolDynamicMesh->EditMesh([&](FDynamicMesh3& Mesh)
	{
		SkeletalMeshToolsHelper::MirrorMorphTargetOnMesh(Mesh, ToolMorphTargetName, *BaseSymmetry, MorphChange);
	});

	if (MorphChange->Vertices.IsEmpty())
	{
		// Nothing to mirror (e.g. all deltas are zero). Skip emitting an empty change.
		delete MorphChange;
		return;
	}

	TUniquePtr<TWrappedToolCommandChange<FMeshMorphTargetChange>> NewChange =
		MakeUnique<TWrappedToolCommandChange<FMeshMorphTargetChange>>();
	NewChange->WrappedChange.Reset(MorphChange);
	NewChange->BeforeModify = [this](bool /*bRevert*/)
	{
		WaitForPendingUndoRedoUpdate();
	};
	NewChange->AfterModify = [this](bool /*bRevert*/)
	{
		bFastDeformSculptMesh = true;
		bFullRefreshSculptMesh = true;
	};

	GetToolManager()->BeginUndoTransaction(LOCTEXT("MirrorMorphTargetTransaction", "Mirror Morph Target"));
	GetToolManager()->EmitObjectChange(ToolDynamicMesh, MoveTemp(NewChange),
		LOCTEXT("MirrorMorphTargetChange", "Mirror Morph Target"));
	GetToolManager()->EndUndoTransaction();

	// EmitObjectChange does not invoke AfterModify on the initial action — only on undo / redo
	// replay. Set the refresh flags directly so the sculpt preview re-poses on the next tick.
	bFastDeformSculptMesh = true;
	bFullRefreshSculptMesh = true;
}

namespace MorphTargetVertexSculptTool::Private
{
	namespace Commands
	{
		constexpr const TCHAR* MirrorMorphTarget = TEXT("MirrorMorphTarget");
	}

	constexpr const TCHAR* const HotkeyHintCommands[] = { Commands::MirrorMorphTarget };
}

void UMorphTargetVertexSculptTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	using namespace MorphTargetVertexSculptTool::Private;

	Super::RegisterActions(ActionSet);

	ActionSet.RegisterAction(this, static_cast<int32>(EStandardToolActions::BaseClientDefinedActionID) + 600,
		Commands::MirrorMorphTarget,
		LOCTEXT("MirrorMorphTargetAction", "Mirror Editing Morph Target"),
		LOCTEXT("MirrorMorphTargetActionTooltip",
			"Mirror the morph target currently selected for editing across the base mesh symmetry plane"),
		EModifierKey::None, EKeys::M,
		[this]() { MirrorEditingMorphTarget(); });
}

void UMorphTargetVertexSculptTool::SetupCommonProperties(const TFunction<void(UMorphTargetEditingToolProperties*)>& InSetupFunction)
{
	EditorToolProperties = NewObject<UMorphTargetEditingToolProperties>(this);
	EditorToolProperties->SetFlags(RF_Transactional);

	InSetupFunction(EditorToolProperties);
	
	AddToolPropertySource(EditorToolProperties);
}

void UMorphTargetVertexSculptTool::HandleSkeletalMeshModified(const TArray<FName>& Payload, const ESkeletalMeshNotifyType InNotifyType)
{
}

bool UMorphTargetVertexSculptTool::IsInputIsolationValidOnOutput() const
{
	return true;
}

void UMorphTargetVertexSculptTool::GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const
{
	for (const TCHAR* CommandName : MorphTargetVertexSculptTool::Private::HotkeyHintCommands)
	{
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ActionCommandsContextName, CommandName);
	}

	// B+drag is owned by BrushEditBehavior (ULocalTwoAxisPropertyEditInputBehavior), not a FUICommandInfo.
	OutHints.Add({ LOCTEXT("HintBrushSize", "Brush Size"), LOCTEXT("HintBrushSizeChord", "B + drag left/right") });
	OutHints.Add({ LOCTEXT("HintBrushStrength", "Brush Strength"), LOCTEXT("HintBrushStrengthChord", "B + drag up/down") });
}

void UMorphTargetVertexSculptTool::PoseSculptMesh(const TArray<FTransform>& ComponentSpaceTransforms, const TMap<FName, float>& MorphTargetWeights)
{
	using namespace SkeletalMeshToolsHelper;
	
	// have to wait for any outstanding stamp/undo update to finish...
	WaitForPendingStampUpdateConst();
	WaitForPendingUndoRedoUpdate();

	TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(ComponentSpaceTransformsRefPose, ComponentSpaceTransforms);

	FDynamicMesh3& SculptMesh = *GetSculptMesh();

	ToolDynamicMesh->ProcessMesh([&](const FDynamicMesh3& InMesh)
		{
			// When isolated, build full-mesh vertex array and remap in WriteFunc
			TArray<int32> FullMeshVertArray;
			if (IsolationSubmesh)
			{
				FullMeshVertArray.Reserve(IsolationSubmesh->GetSubmesh().VertexCount());
				for (int32 SubVID : IsolationSubmesh->GetSubmesh().VertexIndicesItr())
				{
					FullMeshVertArray.Add(IsolationSubmesh->MapVertexToBaseMesh(SubVID));
				}
			}

			auto WriteFunc = [&](FVertInfo VertInfo, const FVector& PosedVertPos)
				{
					int32 SculptVID = IsolationSubmesh ? IsolationSubmesh->MapVertexToSubmesh(VertInfo.VertID) : VertInfo.VertID;
					if (SculptVID != FDynamicMesh3::InvalidID)
					{
						SculptMesh.SetVertex(SculptVID, PosedVertPos);
					}
				};

			GetPosedMesh(WriteFunc, InMesh, BoneMatrices, NAME_None, MorphTargetWeights,
				IsolationSubmesh ? FullMeshVertArray : TArray<int32>());
			FMeshNormals::QuickRecomputeOverlayNormals(SculptMesh);
			
			ComponentSpaceTransformsForPosedMesh = ComponentSpaceTransforms;
			MorphTargetWeightsForPosedMesh = MorphTargetWeights;
		});

	bPosedMeshWithoutEditingMorphDirty = true;

	constexpr bool bNormal = true;
	DynamicMeshComponent->FastNotifyPositionsUpdated(bNormal);
}


void UMorphTargetVertexSculptTool::UpdatePosedMeshWithoutEditingMorph()
{
	using namespace SkeletalMeshToolsHelper;

	WaitForPendingStampUpdateConst();
	WaitForPendingUndoRedoUpdate();

	TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(ComponentSpaceTransformsRefPose, ComponentSpaceTransformsForPosedMesh);

	// Build weights excluding the editing morph target
	TMap<FName, float> WeightsWithoutEditingMorph = MorphTargetWeightsForPosedMesh;
	WeightsWithoutEditingMorph.Remove(ToolMorphTargetName);

	ToolDynamicMesh->ProcessMesh([&](const FDynamicMesh3& InMesh)
		{
			// When isolated, build full-mesh vertex array and remap in WriteFunc
			TArray<int32> FullMeshVertArray;
			if (IsolationSubmesh)
			{
				FullMeshVertArray.Reserve(IsolationSubmesh->GetSubmesh().VertexCount());
				for (int32 SubVID : IsolationSubmesh->GetSubmesh().VertexIndicesItr())
				{
					FullMeshVertArray.Add(IsolationSubmesh->MapVertexToBaseMesh(SubVID));
				}
			}

			auto WriteFunc = [&](FVertInfo VertInfo, const FVector& PosedVertPos)
				{
					int32 WriteVID = IsolationSubmesh ? IsolationSubmesh->MapVertexToSubmesh(VertInfo.VertID) : VertInfo.VertID;
					if (WriteVID != FDynamicMesh3::InvalidID)
					{
						PosedMeshWithoutEditingMorph.SetVertex(WriteVID, PosedVertPos);
					}
				};

			GetPosedMesh(WriteFunc, InMesh, BoneMatrices, NAME_None, WeightsWithoutEditingMorph,
				IsolationSubmesh ? FullMeshVertArray : TArray<int32>());
		});

	bPosedMeshWithoutEditingMorphDirty = false;
}

#undef LOCTEXT_NAMESPACE

