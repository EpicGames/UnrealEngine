// Copyright Epic Games, Inc. All Rights Reserved.

#include "MorphTargetVertexSculptToolCommands.h"

#include "MorphTargetVertexSculptTool.h"

#define LOCTEXT_NAMESPACE "MorphTargetVertexSculptToolCommands"

FMorphTargetVertexSculptToolActionCommands::FMorphTargetVertexSculptToolActionCommands()
	: TInteractiveToolCommands<FMorphTargetVertexSculptToolActionCommands>(
		UMorphTargetVertexSculptTool::ActionCommandsContextName,
		LOCTEXT("ContextDescription", "Skeletal Mesh Morph Target Vertex Sculpt Tool"),
		NAME_None,
		FAppStyle::GetAppStyleSetName())
{
}

void FMorphTargetVertexSculptToolActionCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
	ToolCDOs.Add(GetMutableDefault<UMorphTargetVertexSculptTool>());
}

#undef LOCTEXT_NAMESPACE
