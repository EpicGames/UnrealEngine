// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Tools/InteractiveToolsCommands.h"

/**
 * Per-tool UI commands for UMorphTargetVertexSculptTool.
 *
 * Self-contained: the tool's plugin owns the registration and lifetime, and the morph plugin
 * advertises this command class to a host editor through FExtensionToolDescription::ToolCommandsGetter.
 * Any host that consumes IModelingModeToolExtension already calls BindCommandsForCurrentTool /
 * UnbindActiveCommands on this object during tool start / end (see USkeletalMeshModelingToolsEditorMode::OnToolStarted
 * and UModelingToolsEditorMode::OnToolStarted).
 */
class FMorphTargetVertexSculptToolActionCommands
	: public TInteractiveToolCommands<FMorphTargetVertexSculptToolActionCommands>
{
public:
	FMorphTargetVertexSculptToolActionCommands();

	// TInteractiveToolCommands
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;
};
