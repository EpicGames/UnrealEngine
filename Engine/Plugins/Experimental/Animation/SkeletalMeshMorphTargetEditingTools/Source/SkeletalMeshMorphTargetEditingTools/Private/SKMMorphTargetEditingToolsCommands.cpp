// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKMMorphTargetEditingToolsCommands.h"
#include "SKMMorphTargetEditingToolsStyle.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshMorphTargetEditingToolsCommands"



FSkeletalMeshMorphTargetEditingToolsCommands::FSkeletalMeshMorphTargetEditingToolsCommands()
	: TCommands<FSkeletalMeshMorphTargetEditingToolsCommands>(
		"SkeletalMeshMorphTargetEditingToolsCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "SkeletalMeshMorphTargetEditingToolsCommands", "Skeletal Mesh Morph Target Editing Tools"), // Localized context name for displaying
		NAME_None, // Parent
		FSkeletalMeshMorphTargetEditingToolsStyle::Get().GetStyleSetName() // Icon Style Set
		)
{
}

void FSkeletalMeshMorphTargetEditingToolsCommands::RegisterCommands()
{
	UI_COMMAND(BeginMorphTargetTool, "Morph", "Edit Morph Targets", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(BeginMorphTargetSculptTool, "Sculpt Morph Target", "To use this tool, select a Morph Target for editing from the morph target tab using the radio button", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginMorphTargetMaskTool, "Mask Morph Target", "To use this tool, select a Morph Target for editing from the morph target tab using the radio button", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE

