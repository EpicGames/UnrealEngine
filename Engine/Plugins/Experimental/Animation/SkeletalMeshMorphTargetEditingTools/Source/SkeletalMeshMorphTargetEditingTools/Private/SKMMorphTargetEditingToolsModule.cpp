// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKMMorphTargetEditingToolsModule.h"

#include "MorphTargetMaskTool.h"
#include "MorphTargetVertexSculptTool.h"
#include "MorphTargetVertexSculptToolCommands.h"
#include "SKMMorphTargetEditingToolsCommands.h"
#include "SKMMorphTargetEditingToolsStyle.h"
#include "Features/IModularFeatures.h"
#include "Framework/Commands/Commands.h"
#include "Tools/InteractiveToolsCommands.h"

#define LOCTEXT_NAMESPACE "FSkeletalMeshMorphTargetEditingToolsModule"

void FSkeletalMeshMorphTargetEditingToolsModule::StartupModule()
{
	FSkeletalMeshMorphTargetEditingToolsStyle::Register();
	FSkeletalMeshMorphTargetEditingToolsCommands::Register();
	FMorphTargetVertexSculptToolActionCommands::Register();
	IModularFeatures::Get().RegisterModularFeature(ISkeletalMeshModelingModeToolExtension::GetModularFeatureName(), this);	

}

void FSkeletalMeshMorphTargetEditingToolsModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(ISkeletalMeshModelingModeToolExtension::GetModularFeatureName(), this);
	FMorphTargetVertexSculptToolActionCommands::Unregister();
	FSkeletalMeshMorphTargetEditingToolsCommands::Unregister();
	FSkeletalMeshMorphTargetEditingToolsStyle::Unregister();
}



void FSkeletalMeshMorphTargetEditingToolsModule::GetExtensionTools(const FExtensionToolQueryInfo& QueryInfo,
                                                                   TArray<FExtensionToolDescription>& OutTools)
{
	const FSkeletalMeshMorphTargetEditingToolsCommands& Commands = FSkeletalMeshMorphTargetEditingToolsCommands::Get();
	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("SkeletalMeshMorphTargetSculptTool", "Sculpt Morph Target");
		ToolDesc.ToolCommand = Commands.BeginMorphTargetSculptTool;
		ToolDesc.ToolBuilder = NewObject<UMorphTargetVertexSculptToolBuilder>();
		// Lets the host editor (e.g. USkeletalMeshModelingToolsEditorMode::OnToolStarted) bind / unbind
		// the tool's per-tool hotkeys without taking a compile-time dependency on this plugin.
		ToolDesc.ToolCommandsGetter = []() -> const UE::IInteractiveToolCommandsInterface&
		{
			return FMorphTargetVertexSculptToolActionCommands::Get();
		};
		OutTools.Add(ToolDesc);
	}
	
	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("SkeletalMeshMorphTargetMaskTool", "Mask Morph Target");
		ToolDesc.ToolCommand = Commands.BeginMorphTargetMaskTool;
		ToolDesc.ToolBuilder = NewObject<UMorphTargetMaskToolBuilder>();
		OutTools.Add(ToolDesc);
	}
}

bool FSkeletalMeshMorphTargetEditingToolsModule::GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut)
{
	InfoOut.ExtensionCommand = FSkeletalMeshMorphTargetEditingToolsCommands::Get().BeginMorphTargetTool;
	
	return true;
}

bool FSkeletalMeshMorphTargetEditingToolsModule::GetExtensionToolTargets(TArray<TSubclassOf<UToolTargetFactory>>& ToolTargetFactoriesOut)
{
	return false;
}




#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FSkeletalMeshMorphTargetEditingToolsModule, SkeletalMeshMorphTargetEditingTools)