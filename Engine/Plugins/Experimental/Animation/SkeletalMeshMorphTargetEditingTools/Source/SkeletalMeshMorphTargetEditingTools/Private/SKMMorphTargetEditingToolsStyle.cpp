// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKMMorphTargetEditingToolsStyle.h"

#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"


FSkeletalMeshMorphTargetEditingToolsStyle& FSkeletalMeshMorphTargetEditingToolsStyle::Get()
{
	static FSkeletalMeshMorphTargetEditingToolsStyle Instance;
	return Instance;
}

void FSkeletalMeshMorphTargetEditingToolsStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FSkeletalMeshMorphTargetEditingToolsStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FSkeletalMeshMorphTargetEditingToolsStyle::FSkeletalMeshMorphTargetEditingToolsStyle()
	: FSlateStyleSet("SkeletalMeshMorphTargetEditingToolsStyle")
{
	static const FVector2D IconSize10x10(10.0f, 10.0f);
	static const FVector2D IconSize16x12(16.0f, 12.0f);
	static const FVector2D IconSize16x16(16.0f, 16.0f);
	static const FVector2D IconSize20x20(20.0f, 20.0f);
	static const FVector2D IconSize64x64(64.0f, 64.0f);

	static const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

	{
		FSlateStyleSet::SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/Animation/SkeletalMeshMorphTargetEditingTools/Resources"));
		Set("SkeletalMeshMorphTargetEditingToolsCommands.BeginMorphTargetTool", new IMAGE_BRUSH_SVG("MorphTarget_20", IconSize20x20));
	}

	{
		FSlateStyleSet::SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/ModelingToolsEditorMode/Content"));
		Set("SkeletalMeshMorphTargetEditingToolsCommands.BeginMorphTargetSculptTool", new IMAGE_BRUSH("Icons/Sculpt_40x", IconSize20x20));	
	}
}
