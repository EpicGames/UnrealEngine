// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "UObject/Interface.h"

#include "IMorphTargetEditingToolInterface.generated.h"

class UMorphTargetEditingToolProperties;

UINTERFACE(MinimalAPI)
class UMorphTargetEditingToolInterface : public UInterface
{
	GENERATED_BODY()
};

class IMorphTargetEditingToolInterface : public IInterface
{
	GENERATED_BODY()
public:
	virtual void SetupMorphEditingToolCommon();
	virtual void ShutdownMorphEditingToolCommon();
	virtual void SetupCommonProperties(const TFunction<void(UMorphTargetEditingToolProperties*)>& InSetupFunction) = 0;
};

UCLASS()
class UMorphTargetEditingToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category = "MorphTarget", meta = (NoResetToDefault, DisplayName = "Name"))
	FName EditMorphTargetName;
};
