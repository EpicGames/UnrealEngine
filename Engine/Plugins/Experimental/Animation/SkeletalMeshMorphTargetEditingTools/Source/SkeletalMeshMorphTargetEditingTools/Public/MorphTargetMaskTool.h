// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMorphTargetEditingToolInterface.h"

#include "DynamicSubmesh3.h"
#include "MeshVertexAttributePaintToolBase.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "SkeletalMesh/IHotkeyHintProvider.h"
#include "SkeletalMesh/ISkeletalMeshGeometryIsolationAwareTool.h"

#include "MorphTargetMaskTool.generated.h"

/**
 * MorphTarget Vertex Sculpt Tool Builder
 */
UCLASS()
class UMorphTargetMaskToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UCLASS(MinimalAPI)
class UMorphTargetMaskToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	void Setup(UMorphTargetMaskTool* InTool, TArray<FName> InVertexAttributes);
	void Shutdown(UMorphTargetMaskTool* InTool , EToolShutdownType InShutdownType);
	bool ShouldSaveMask() const;
	bool ShouldSaveNewMask() const;
	
	UPROPERTY(EditAnywhere, Category = "Mask Settings", meta = (GetOptions = GetMorphTargets,  NoResetToDefault))
	FName BaseMorphTarget = NAME_None;

	UFUNCTION()
	const TArray<FName>& GetMorphTargets();

	TArray<FName> MorphTargets;

	UPROPERTY(EditAnywhere, Category = "Mask Settings", meta = (DisplayName = "Vertex Attribute", GetOptions = GetVertexAttributeOptions,  NoResetToDefault))
	FName VertexAttributeOption = CreateNewVertexAttributeOption;
	
	UFUNCTION()
	const TArray<FName>& GetVertexAttributeOptions();

	TArray<FName> VertexAttributeOptions;
	TArray<FName> VertexAttributes;

	FString PropertySetCacheIdentifier;	

	static const TCHAR* CreateNewVertexAttributeOption;
	static const TCHAR* DoNotSaveVertexAttributeOption;

	TOptional<int32> GetVertexAttributeIndex();
};

/**
* MorphTarget Sculpt Tool
*/
UCLASS()
class  UMorphTargetMaskTool: 
	public UMeshVertexAttributePaintToolBase,
	public IMorphTargetEditingToolInterface,
	public ISkeletalMeshGeometryIsolationAwareTool,
	public IHotkeyHintProvider
{
	GENERATED_BODY()

	virtual bool SetupToolMesh(FDynamicMesh3& InOutToolMesh, int32& OutInitialAttributeIndex) override;
	virtual void CommitToolMesh(FDynamicMesh3& InToolMesh) override;
	virtual void UpdatePreview(const TSet<int32>* TrianglesToUpdate = nullptr, const TArray<int32>* VerticesToUpdate = nullptr) override;
public:
	FName GetEditingMorphTarget();
	FName GetBaseMorphTarget();
	virtual TMap<FName, float> GetMorphTargetWeights();
	virtual const TArray<FTransform>& GetComponentSpaceBoneTransforms();
	virtual void ToggleBoneManipulation(bool bEnable);


	// UMeshVertexSculptTool overrides
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void OnTick(float DeltaTime) override;

	// ISkeletalMeshGeometryIsolationAwareTool
	virtual bool IsInputIsolationValidOnOutput() const override;

	// IHotkeyHintProvider
	virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const override;

protected:


	void HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload);

	// IMorphTargetEditingTool overrides
	virtual void SetupCommonProperties(const TFunction<void(UMorphTargetEditingToolProperties*)>& InSetupFunction) override;
	
	void PosePreviewMesh(
		const TArray<FTransform>& ComponentSpaceTransforms,
		const TMap<FName, float>& MorphTargetWeights,
		const TSet<int32>* TrianglesToUpdate = nullptr,
		const TArray<int32>* VerticesToUpdate = nullptr
		);

	UPROPERTY()
	TObjectPtr<class UMorphTargetEditingToolProperties> EditorToolProperties;

	UPROPERTY()
	TObjectPtr<class UMorphTargetMaskToolProperties> MaskToolProperties;


	
	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext = nullptr;

	SkeletalMeshToolsHelper::FPoseChangeDetector PoseChangeDetector;

	FDynamicMesh3 MaskToolCommitMesh;

	// Geometry isolation: submesh mapping for vertex ID translation (null when no isolation)
	TOptional<UE::Geometry::FDynamicSubmesh3> IsolationSubmesh;

	TArray<FTransform> ComponentSpaceTransformsRefPose;
	TMap<FName, float> MorphTargetZeroWeights;
	
	FName ToolMorphTargetName;
	FName ToolMorphTargetBackupAttributeName;

	TArray<FTransform> DefaultComponentSpaceBoneTransforms;

	TValueWatcher<FName> BaseMorphTargetWatcher;
	TValueWatcher<FName> VertexAttributeOptionWatcher;

	bool bFastDeformPreviewMesh = false;
	bool bRebuildOctree = false;

	bool bMeshSelectorNeedsUpdate = false;

	
	int32 TempPaintLayerAttributeIndex = INDEX_NONE;
	int32 CurrentAttributeIndex = INDEX_NONE;
};





