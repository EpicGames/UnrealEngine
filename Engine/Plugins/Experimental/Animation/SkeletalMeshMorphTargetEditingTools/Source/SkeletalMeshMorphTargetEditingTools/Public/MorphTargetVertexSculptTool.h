// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMorphTargetEditingToolInterface.h"
#include "MeshVertexSculptTool.h"
#include "DynamicSubmesh3.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "MeshDescription.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "SkeletalMesh/IHotkeyHintProvider.h"
#include "SkeletalMesh/ISkeletalMeshGeometryIsolationAwareTool.h"

#include "MorphTargetVertexSculptTool.generated.h"

/**
 * MorphTarget Vertex Sculpt Tool Builder
 */
UCLASS()
class UMorphTargetVertexSculptToolBuilder : public UMeshVertexSculptToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


/**
* MorphTarget Sculpt Tool
*/
UCLASS()
class UMorphTargetVertexSculptTool : 
	public UMeshVertexSculptTool,
	public ISkeletalMeshEditingInterface,
	public IMorphTargetEditingToolInterface,
	public ISkeletalMeshGeometryIsolationAwareTool,
	public IHotkeyHintProvider
{
	GENERATED_BODY()

public:
	static constexpr const TCHAR* ActionCommandsContextName = TEXT("SkeletalMeshMorphTargetVertexSculptTool");

	virtual FName GetEditingMorphTarget();
	virtual TMap<FName, float> GetMorphTargetWeights();
	virtual const TArray<FTransform>& GetComponentSpaceBoneTransforms();
	virtual void ToggleBoneManipulation(bool bEnable);

	// Mirrors the editing morph target's deltas across the base-mesh symmetry plane in place,
	// emits a single undoable change, and queues a sculpt-mesh refresh on the next tick.
	// No-op while a stroke is in progress, no editing morph is selected, or no base-mesh symmetry is available.
	void MirrorEditingMorphTarget();

	// True when MirrorEditingMorphTarget would actually do something.
	bool CanMirrorEditingMorphTarget() const;


	// UMeshSculptToolBase overrides
	virtual void InitializeSculptMeshComponent(UBaseDynamicMeshComponent* Component, AActor* Actor) override;
	virtual void SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior) override;

	// UMeshVertexSculptTool overrides
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology) override;
	virtual void OnTick(float DeltaTime) override;
	virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	virtual void OnBeginStroke(const FRay& WorldRay) override;
	virtual void OnEndStroke() override;

	// UInteractiveTool overrides
	virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	// ISkeletalMeshGeometryIsolationAwareTool
	virtual bool IsInputIsolationValidOnOutput() const override;

	// IHotkeyHintProvider
	virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const override;

protected:
	friend class FMorphTargetVertexSculptNonSymmetricChange;
	virtual void RegisterBrushes() override;

	void HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload);

	// IMorphTargetEditingTool overrides
	virtual void SetupCommonProperties(const TFunction<void(UMorphTargetEditingToolProperties*)>& InSetupFunction) override;
	
	// ISkeletalMeshEditingInterface
	virtual void HandleSkeletalMeshModified(const TArray<FName>& Payload, const ESkeletalMeshNotifyType InNotifyType) override;
	
	
	
	void PoseSculptMesh(const TArray<FTransform>& ComponentSpaceTransforms, const TMap<FName, float>& MorphTargetWeights);

	UPROPERTY()
	TObjectPtr<class UMorphTargetEditingToolProperties> EditorToolProperties;

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext = nullptr;

	TFunction<const FDynamicMesh3*()> GetMeshWithoutCurrentMorphFunc;
	FDynamicMesh3 PosedMeshWithoutEditingMorph;
	bool bPosedMeshWithoutEditingMorphDirty = true;
	void UpdatePosedMeshWithoutEditingMorph();
	
	SkeletalMeshToolsHelper::FPoseChangeDetector PoseChangeDetector;

	

	FDelegateHandle OnToolMeshChangedDelegate;

	UPROPERTY()
	TObjectPtr<UDynamicMesh> ToolDynamicMesh;

	// Geometry isolation: submesh mapping for vertex ID translation (null when no isolation)
	TOptional<UE::Geometry::FDynamicSubmesh3> IsolationSubmesh;
	
	TArray<FTransform> ComponentSpaceTransformsRefPose;
	
	FName ToolMorphTargetName;


	TArray<FTransform> DefaultComponentSpaceBoneTransforms;
	
	TArray<FTransform> ComponentSpaceTransformsForPosedMesh;
	TMap<FName, float> MorphTargetWeightsForPosedMesh;

	bool bFastDeformSculptMesh = false;
	bool bFullRefreshSculptMesh = false;
};





