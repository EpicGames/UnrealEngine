// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "SkeletalMeshModelingModeToolExtensions.h"

class FUICommandInfo;

class FSkeletalMeshMorphTargetEditingToolsModule : public IModuleInterface, public ISkeletalMeshModelingModeToolExtension 
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	// ISkeletalMeshModelingModeToolExtension
	virtual FText GetExtensionName() override { return FText::FromString(TEXT("SkeletalMeshMorphTargetEditingTools"));}

	virtual FText GetToolSectionName() override { return FText::FromString(TEXT("Morph"));};

	virtual void GetExtensionTools(const FExtensionToolQueryInfo& QueryInfo, TArray<FExtensionToolDescription>& OutTools) override;
	
	virtual bool GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut) override;

	virtual bool GetExtensionToolTargets(TArray<TSubclassOf<UToolTargetFactory>>& ToolTargetFactoriesOut) override;

};
