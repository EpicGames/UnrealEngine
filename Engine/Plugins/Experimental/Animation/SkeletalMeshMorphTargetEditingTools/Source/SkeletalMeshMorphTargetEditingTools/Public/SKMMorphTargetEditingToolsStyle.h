// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"


class FSkeletalMeshMorphTargetEditingToolsStyle
	: public FSlateStyleSet
{
public:
	static FSkeletalMeshMorphTargetEditingToolsStyle& Get();

protected:
	friend class FSkeletalMeshMorphTargetEditingToolsModule;

	static void Register();
	static void Unregister();

private:
	FSkeletalMeshMorphTargetEditingToolsStyle();
};