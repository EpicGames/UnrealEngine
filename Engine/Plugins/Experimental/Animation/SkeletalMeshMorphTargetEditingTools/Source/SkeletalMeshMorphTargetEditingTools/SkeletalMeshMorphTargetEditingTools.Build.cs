// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SkeletalMeshMorphTargetEditingTools : ModuleRules
{
	public SkeletalMeshMorphTargetEditingTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
				"ModelingToolsEditorMode",
				"SkeletalMeshModelingTools", 
				"MeshModelingToolsEditorOnly", 
				"MeshModelingToolsEditorOnlyExp"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore", 
				"ModelingComponents", 
				"MeshConversion", 
				"MeshModelingTools", 
				"MeshModelingToolsEditorOnly", 
				"InteractiveToolsFramework",
				"SkeletalMeshUtilitiesCommon",
				"DynamicMesh",
				"GeometryCore",
				"SkeletalMeshModifiers",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"ModelingComponentsEditorOnly",
				"UnrealEd",
				"Persona",
				"GeometryFramework", 
				"SkeletalMeshEditor"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
		
		ShortName = "SKMMorphTools";
	}
}
